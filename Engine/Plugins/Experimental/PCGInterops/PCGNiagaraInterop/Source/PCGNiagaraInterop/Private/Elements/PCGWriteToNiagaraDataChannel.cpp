// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGWriteToNiagaraDataChannel.h"

#include "Helpers/PCGAttributeNiagaraTraits.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGLogErrors.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataChannelPublic.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "PCGWriteToNDCElement"

#if WITH_EDITOR
FName UPCGWriteToNiagaraDataChannelSettings::GetDefaultNodeName() const
{
	return FName(TEXT("WriteToNiagaraDataChannel"));
}

FText UPCGWriteToNiagaraDataChannelSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Write To Niagara Data Channel");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGWriteToNiagaraDataChannelSettings::CreateElement() const
{
	return MakeShared<FPCGWriteToNiagaraDataChannelElement>();
}

TArray<FPCGPinProperties> UPCGWriteToNiagaraDataChannelSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGWriteToNiagaraDataChannelSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return Properties;
}

bool FPCGWriteToNiagaraDataChannelElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWriteToNiagaraDataChannelElement::Execute);

	FPCGWriteToNiagaraDataChannelContext* Context = reinterpret_cast<FPCGWriteToNiagaraDataChannelContext*>(InContext);
	check(InContext);

	const UPCGWriteToNiagaraDataChannelSettings* Settings = InContext->GetInputSettings<UPCGWriteToNiagaraDataChannelSettings>();
	check(Settings);
	
	if (Settings->DataChannel.IsNull())
	{
		return true;
	}

	if (!Context->WasLoadRequested())
	{
		return Context->RequestResourceLoad(Context, {Settings->DataChannel.ToSoftObjectPath()}, !Settings->bSynchronousLoad);
	}
	else
	{
		return true;
	}
}

bool FPCGWriteToNiagaraDataChannelElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWriteToNiagaraDataChannelElement::Execute);

	check(InContext);

	const UPCGWriteToNiagaraDataChannelSettings* Settings = InContext->GetInputSettings<UPCGWriteToNiagaraDataChannelSettings>();
	check(Settings);

	// Passthrough input
	InContext->OutputData = InContext->InputData;

	UNiagaraDataChannelAsset* DataChannelAsset = Settings->DataChannel.Get();
	UNiagaraDataChannel* DataChannel = DataChannelAsset ? DataChannelAsset->Get() : nullptr;
	if (!DataChannel)
	{
		return true;
	}

	TArray<FName, TInlineAllocator<16>> MatchedNiagaraVars;
	MatchedNiagaraVars.Reserve(Settings->NiagaraVariablesPCGAttributeMapping.Num());
	bool bUnmatchedVarsWereWarned = false;

	int InputNum = 0;

	UWorld* World = InContext->ExecutionSource.IsValid() ? InContext->ExecutionSource->GetExecutionState().GetWorld() : nullptr;
	if (!World)
	{
		return true;
	}

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		InputNum++;
		int32 Count = -1;

		using FAccessorAndKeysTuple = TTuple<const FNiagaraVariableBase&, TUniquePtr<const IPCGAttributeAccessor>, TUniquePtr<const IPCGAttributeAccessorKeys>>;
		TArray<FAccessorAndKeysTuple> AccessorAndKeys;
		for (const FNiagaraDataChannelVariable& NiagaraVar : DataChannel->GetVariables())
		{
			const FName NiagaraVarName = NiagaraVar.GetName();
			if (const FPCGAttributePropertyInputSelector* Selector = Settings->NiagaraVariablesPCGAttributeMapping.Find(NiagaraVarName))
			{
				if (!bUnmatchedVarsWereWarned)
				{
					MatchedNiagaraVars.Add(NiagaraVarName);
				}

				const FPCGAttributePropertyInputSelector FixedSelector = Selector->CopyAndFixLast(Input.Data);

				TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, FixedSelector);
				TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, FixedSelector);

				if (!Accessor || !Keys)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(FixedSelector, InContext);
					continue;
				}

				// Verify that types are compatible
				if (!PCGAttributeNiagaraTraits::AreTypesCompatible(Accessor->GetUnderlyingType(), NiagaraVar, /*bPCGToNiagara=*/true))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchNiagaraVarType", "Niagara variable {0} is not compatible with attribute {1} ({2})."), 
						FText::FromName(NiagaraVarName), FixedSelector.GetDisplayText(), PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType())), InContext);
					continue;
				}

				Count = FMath::Max(Count, Keys->GetNum());
				AccessorAndKeys.Emplace(NiagaraVar, std::move(Accessor), std::move(Keys));
			}
		}

		if (!bUnmatchedVarsWereWarned && MatchedNiagaraVars.Num() != Settings->NiagaraVariablesPCGAttributeMapping.Num())
		{
			for (const auto& [NiagaraVarName, Selector] : Settings->NiagaraVariablesPCGAttributeMapping)
			{
				if (!MatchedNiagaraVars.Contains(NiagaraVarName))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchNiagaraVarName", "Niagara variable {0} was not found in the data channel."), FText::FromName(NiagaraVarName)), InContext);
				}
			}
		}

		bUnmatchedVarsWereWarned = true;

		if (AccessorAndKeys.IsEmpty() || Count <= 0)
		{
			continue;
		}

		FNiagaraDataChannelSearchParameters SearchParameters;
		AActor* TargetActor = InContext->GetTargetActor(nullptr);
		SearchParameters.OwningComponent = TargetActor ? TargetActor->GetRootComponent() : nullptr;

		const FString DebugSource = FString::Format(TEXT("PCGWriteToNiagaraChannel - {0} - Input {1}"), { InContext->Node ? InContext->Node->GetName() : TEXT("Unknown node"), FString::FromInt(InputNum - 1) });

		UNiagaraDataChannelWriter* NiagaraWriter = UNiagaraDataChannelLibrary::CreateDataChannelWriter(World, DataChannel, std::move(SearchParameters), Count, Settings->bVisibleToGame, Settings->bVisibleToCPU, Settings->bVisibleToGPU, DebugSource);

		if (!NiagaraWriter)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("FailToCreateWriter", "Failed to create Niagara Data Channel Writer"), InContext);
			continue;
		}

		for (const FAccessorAndKeysTuple& It : AccessorAndKeys)
		{
			const FNiagaraVariableBase& NiagaraVar = It.Get<0>();
			const TUniquePtr<const IPCGAttributeAccessor>& Accessor = It.Get<1>();
			const TUniquePtr<const IPCGAttributeAccessorKeys>& Keys = It.Get<2>();
			
			PCGAttributeNiagaraTraits::CallbackWithNiagaraType(NiagaraVar, [&NiagaraVar, &Accessor, &Keys, NiagaraWriter]<typename NiagaraType>(NiagaraType)
			{
				// PCG doesn't support linear color natively, so read it from a FVector4.
				using PCGType = std::conditional_t<std::is_same_v<NiagaraType, FLinearColor>, FVector4, NiagaraType>;
				return PCGMetadataElementCommon::ApplyOnAccessor<PCGType>(*Keys, *Accessor, [NiagaraWriter, &NiagaraVar](const PCGType& InValue, int32 Index)
					{
						if constexpr (std::is_same_v<NiagaraType, FLinearColor>)
						{
							NiagaraWriter->WriteData(NiagaraVar, Index, FLinearColor{ InValue });
						}
						else if constexpr (std::is_same_v<NiagaraType, bool>)
						{
							// Need to use the FNiagaraBool struct
							NiagaraWriter->WriteData(NiagaraVar, Index, FNiagaraBool(InValue));
						}
						else
						{
							NiagaraWriter->WriteData(NiagaraVar, Index, InValue);
						}
					}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
			});
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
