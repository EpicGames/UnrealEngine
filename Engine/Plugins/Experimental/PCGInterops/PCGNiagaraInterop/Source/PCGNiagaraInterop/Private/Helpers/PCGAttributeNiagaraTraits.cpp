// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGAttributeNiagaraTraits.h"

bool PCGAttributeNiagaraTraits::AreTypesCompatible(uint16 PCGType, const FNiagaraVariableBase& NiagaraVar, bool bPCGToNiagara)
{
	return CallbackWithNiagaraType(NiagaraVar, [PCGType, bPCGToNiagara]<typename T>(T) -> bool
	{
		if constexpr (std::is_same_v<T, FNiagaraSpawnInfo>)
		{
			// Not supported at the moment
			return false;
		}
		else
		{
			using NiagaraType = std::conditional_t<std::is_same_v<T, FLinearColor>, FVector4, T>;
			return bPCGToNiagara
				? PCG::Private::IsBroadcastableOrConstructible(PCGType, PCG::Private::MetadataTypes<NiagaraType>::Id)
				: PCG::Private::IsBroadcastableOrConstructible(PCG::Private::MetadataTypes<NiagaraType>::Id, PCGType);
		}
	});
}