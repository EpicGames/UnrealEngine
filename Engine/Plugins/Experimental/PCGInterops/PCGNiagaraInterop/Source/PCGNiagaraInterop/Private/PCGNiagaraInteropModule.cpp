// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGNiagaraInteropModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPCGNiagaraInteropModule, PCGNiagaraInterop);

DEFINE_LOG_CATEGORY(LogPCGNiagaraInterop);
