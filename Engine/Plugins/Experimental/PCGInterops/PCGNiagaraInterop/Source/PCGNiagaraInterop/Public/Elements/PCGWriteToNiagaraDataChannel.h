// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGWriteToNiagaraDataChannel.generated.h"

class UNiagaraDataChannelAsset;

/**
* Allow writing attributes to a Niagara Data Channel.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGWriteToNiagaraDataChannelSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	TSoftObjectPtr<UNiagaraDataChannelAsset> DataChannel;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TMap<FName, FPCGAttributePropertyInputSelector> NiagaraVariablesPCGAttributeMapping;

	/** Data written to this data channel is visible to Blueprint and C++ logic reading from it */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Visibility", meta = (PCG_Overridable))
	bool bVisibleToGame = true;

	/** Data written to this data channel is visible to Niagara CPU emitters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Visibility", meta = (PCG_Overridable))
	bool bVisibleToCPU = true;

	/** Data written to this data channel is visible to Niagara GPU emitters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Visibility", meta = (PCG_Overridable))
	bool bVisibleToGPU = false;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGWriteToNiagaraDataChannelContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGWriteToNiagaraDataChannelElement : public IPCGElementWithCustomContext<FPCGWriteToNiagaraDataChannelContext>
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

