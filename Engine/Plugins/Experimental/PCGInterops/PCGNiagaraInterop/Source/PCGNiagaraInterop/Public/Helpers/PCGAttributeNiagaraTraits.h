// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "NiagaraTypes.h"

namespace PCGAttributeNiagaraTraits
{
	bool AreTypesCompatible(uint16 PCGType, const FNiagaraVariableBase& NiagaraVar, bool bPCGToNiagara);

	template <typename Func, typename  ...Args>
	decltype(auto) CallbackWithNiagaraType(const FNiagaraVariableBase& NiagaraVar, Func Callback, Args&& ...InArgs)
	{
		using ReturnType = decltype(Callback(double{}, std::forward<Args>(InArgs)...));
		const FNiagaraTypeDefinition& NiagaraType = NiagaraVar.GetType();

		if (NiagaraType == FNiagaraTypeHelper::GetVector2DDef())
		{
			return Callback(FVector2D{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeHelper::GetVectorDef() || NiagaraType == FNiagaraTypeDefinition::GetPositionDef())
		{
			return Callback(FVector{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeHelper::GetVector4Def())
		{
			return Callback(FVector4{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeDefinition::GetColorDef())
		{
			return Callback(FLinearColor{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeHelper::GetQuatDef())
		{
			return Callback(FQuat{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeHelper::GetDoubleDef())
		{
			return Callback(double{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeDefinition::GetIntDef())
		{
			return Callback(int32{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeDefinition::GetIDDef())
		{
			return Callback(FNiagaraID{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeDefinition::GetBoolDef())
		{
			return Callback(bool{}, std::forward<Args>(InArgs)...);
		}
		else if (NiagaraType == FNiagaraTypeDefinition(FNiagaraSpawnInfo::StaticStruct()))
		{
			return Callback(FNiagaraSpawnInfo{}, std::forward<Args>(InArgs)...);
		}
		else
		{
			if constexpr (std::is_same_v<ReturnType, void>)
			{
				return;
			}
			else
			{
				return ReturnType{};
			}
		}
	}
}